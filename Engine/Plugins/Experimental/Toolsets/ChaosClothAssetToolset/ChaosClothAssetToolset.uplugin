{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ChaosClothAsset Toolset",
	"Description": "AI agent tools for creating and assigning ChaosClothAsset clothing to skeletal meshes.",
	"Category": "Cloth",
	"EditorOnly": true,
	"IsExperimentalVersion": true,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "ChaosClothAssetToolset",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		},
		{
			"Name": "ChaosClothAsset",
			"Enabled": true
		},
		{
			"Name": "ChaosClothAssetEditorCore",
			"Enabled": true
		}
	]
}
