// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosClothAssetToolset : ModuleRules
{
	public ChaosClothAssetToolset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		IWYUSupport = IWYUSupport.Full;

		PublicDependencyModuleNames.Add("Core");

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"ChaosClothAssetEditor",          // FLegacyClothingConverter
			"ChaosClothAssetEngine",          // UChaosClothAssetBase, UChaosClothAssetSKMClothingAsset
			"ClothingSystemEditorInterface",  // FClothingSystemEditorInterfaceModule, UClothingAssetFactoryBase, FScopedSkeletalMeshPostEditChange
			"ClothingSystemRuntimeCommon",    // UClothingAssetCommon
			"ClothingSystemRuntimeInterface", // UClothingAssetBase
			"CoreUObject",
			"Engine",
			"SkeletalMeshEditor",
			"ToolsetRegistry",
			"UnrealEd",                       // FScopedTransaction
		});
	}
}
