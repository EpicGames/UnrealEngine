// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosClothAsset/ClothAssetToolset.h"
#include "ToolsetRegistry/AgentSkill.h"

#include "ChaosClothAssetConversionSkill.generated.h"

/**
 * Agent skill that documents the end-to-end workflow for converting a legacy UClothingAssetCommon
 * on a SkeletalMesh into a new UChaosClothAsset using the ChaosClothAsset toolset.
 */
UCLASS()
class UChaosClothAssetConversionSkill : public UAgentSkill
{
	GENERATED_BODY()

public:
	UChaosClothAssetConversionSkill()
	{
		Description = TEXT("Convert a legacy UClothingAssetCommon on a SkeletalMesh into a new UChaosClothAsset, then bind it to skeletal mesh sections.");
		Instructions = TEXT(
			"Workflow to convert legacy Chaos cloth to the new ChaosClothAsset format:\n"
			"1. Call ListClothingAssets(SkeletalMeshPath) and locate entries where "
			"bRequiresMatchingLodIndex == false (those are legacy UClothingAssetCommon).\n"
			"2. For each legacy asset, call ConvertClothingAssetCommonToChaosClothAsset("
			"SkeletalMeshPath, ClothingAssetName, OutputPackagePath, AssetName). "
			"Pass OutputPackagePath as a content folder like \"/Game/Cloth/\" and AssetName "
			"as an empty string to use the default \"CA_Converted_<source>\" name.\n"
			"3. The returned path points to a new UChaosClothAsset. Its embedded Dataflow graph "
			"has been baked from the legacy config; the source UClothingAssetCommon is referenced "
			"by the ClothingAssetImport node. Use that node's Reimport button to refresh the "
			"geometry after edits to the legacy asset.\n"
			"4. To bind the new asset to skeletal mesh sections, call CreateClothingAsset("
			"SkeletalMeshPath, NewClothAssetPath) to attach it, then AssignClothingToSection(...) "
			"for each desired LOD/section. Pass ClothingLodIndex equal to LodIndex because "
			"UChaosClothAsset-derived assets have bRequiresMatchingLodIndex == true.\n"
			"5. Optionally call RemoveClothingFromSection(...) to unbind the legacy clothing "
			"once the new binding has been validated."
		);
	}
};
