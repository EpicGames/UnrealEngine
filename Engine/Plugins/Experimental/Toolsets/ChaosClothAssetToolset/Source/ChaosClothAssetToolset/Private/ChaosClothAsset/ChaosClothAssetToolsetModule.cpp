// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothAssetToolset.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

class FChaosClothAssetToolsetModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		UToolsetRegistry::RegisterToolsetClass(UChaosClothAssetToolset::StaticClass());
	}

	virtual void ShutdownModule() override
	{
		UToolsetRegistry::UnregisterToolsetClass(UChaosClothAssetToolset::StaticClass());
	}
};

IMPLEMENT_MODULE(FChaosClothAssetToolsetModule, ChaosClothAssetToolset);
