// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosClothAsset/ClothAssetToolset.h"

#include "ChaosClothAsset/ClothAsset.h"
#include "ChaosClothAsset/ClothAssetBase.h"
#include "ChaosClothAsset/ClothAssetSKMClothingAsset.h"
#include "ChaosClothAsset/LegacyClothingConverter.h"
#include "ClothingAsset.h"
#include "ClothingAssetBase.h"
#include "ClothingAssetFactoryInterface.h"
#include "ClothingSystemEditorInterfaceModule.h"
#include "Engine/SkeletalMesh.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Modules/ModuleManager.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "ScopedTransaction.h"
#include "SkeletalMeshClothingSystemUtilities.h"
#include "SkeletalMeshTypes.h"
#include "UObject/SoftObjectPath.h"

namespace UE::ClothAssetToolset::Private
{
void RaiseError(const FString& Message)
{
	UKismetSystemLibrary::RaiseScriptError(FString::Printf(TEXT("ClothAssetToolset: %s"), *Message));
}
}


TArray<FString> UChaosClothAssetToolset::CreateClothingAsset(
	const FString& SkeletalMeshPath, const FString& ChaosClothAssetPath)
{
	using namespace UE::ClothAssetToolset::Private;

	TArray<FString> Result;

	USkeletalMesh* Mesh = Cast<USkeletalMesh>(FSoftObjectPath(SkeletalMeshPath).TryLoad());
	if (!Mesh)
	{
		RaiseError(FString::Printf(TEXT("Failed to load skeletal mesh '%s'"), *SkeletalMeshPath));
		return Result;
	}

	UChaosClothAssetBase* ClothAsset = Cast<UChaosClothAssetBase>(FSoftObjectPath(ChaosClothAssetPath).TryLoad());
	if (!ClothAsset)
	{
		RaiseError(FString::Printf(TEXT("Failed to load ChaosClothAsset '%s'"), *ChaosClothAssetPath));
		return Result;
	}

	FClothingSystemEditorInterfaceModule& ClothingEditorModule =
		FModuleManager::LoadModuleChecked<FClothingSystemEditorInterfaceModule>("ClothingSystemEditorInterface");

	UClothingAssetFactoryBase* Factory = nullptr;
	for (UClothingAssetFactoryBase* Candidate : ClothingEditorModule.GetClothingAssetFactories())
	{
		if (Candidate && Candidate->GetSupportedSourceAssetType() == UChaosClothAssetBase::StaticClass())
		{
			Factory = Candidate;
			break;
		}
	}

	if (!Factory)
	{
		RaiseError(TEXT("No clothing asset factory found that supports UChaosClothAssetBase."));
		return Result;
	}

	FScopedTransaction Transaction(
		NSLOCTEXT("ChaosClothAssetToolset", "CreateClothingAsset", "Create ChaosCloth Asset"));
	Mesh->Modify();

	TArray<UClothingAssetBase*> CreatedAssets = Factory->CreateFromSourceAsset(Mesh, ClothAsset);
	if (CreatedAssets.IsEmpty())
	{
		RaiseError(TEXT("Factory returned no clothing assets."));
		return Result;
	}

	Result.Reserve(CreatedAssets.Num());
	for (UClothingAssetBase* Asset : CreatedAssets)
	{
		if (Asset)
		{
			Mesh->AddClothingAsset(Asset);
			Result.Add(Asset->GetName());
		}
	}

	return Result;
}

bool UChaosClothAssetToolset::AssignClothingToSection(
	const FString& SkeletalMeshPath, const FString& ClothingAssetName, int32 LodIndex, int32 SectionIndex, int32 ClothingLodIndex)
{
	using namespace UE::ClothAssetToolset::Private;
	USkeletalMesh* Mesh = Cast<USkeletalMesh>(FSoftObjectPath(SkeletalMeshPath).TryLoad());
	if (!Mesh)
	{
		RaiseError(FString::Printf(TEXT("Failed to load skeletal mesh '%s'"), *SkeletalMeshPath));
		return false;
	}

	UClothingAssetBase* Asset = nullptr;
	for (UClothingAssetBase* Candidate : Mesh->GetMeshClothingAssets())
	{
		if (Candidate && Candidate->GetName() == ClothingAssetName)
		{
			Asset = Candidate;
			break;
		}
	}

	if (!Asset)
	{
		RaiseError(FString::Printf(TEXT("No clothing asset found with name '%s'"), *ClothingAssetName));
		return false;
	}
	FString ErrorMessage;
	const bool bSuccess = FSkeletalMeshClothingSystemUtilities::AssignClothingToSection(Mesh, Asset, LodIndex, SectionIndex, ClothingLodIndex, &ErrorMessage);
	if (!bSuccess && !ErrorMessage.IsEmpty())
	{
		RaiseError(ErrorMessage);
	}
	return bSuccess;
}

bool UChaosClothAssetToolset::RemoveClothingFromSection(
	const FString& SkeletalMeshPath, int32 LodIndex, int32 SectionIndex)
{
	using namespace UE::ClothAssetToolset::Private;
	USkeletalMesh* Mesh = Cast<USkeletalMesh>(FSoftObjectPath(SkeletalMeshPath).TryLoad());
	FString ErrorMessage;
	const bool bSuccess = FSkeletalMeshClothingSystemUtilities::RemoveClothingFromSection(Mesh, LodIndex, SectionIndex, &ErrorMessage);
	if (!bSuccess && !ErrorMessage.IsEmpty())
	{
		RaiseError(ErrorMessage);
	}
	return bSuccess;
}

TArray<FClothingAssetInfo> UChaosClothAssetToolset::ListClothingAssets(const FString& SkeletalMeshPath)
{
	TArray<FClothingAssetInfo> Result;

	const USkeletalMesh* Mesh = Cast<USkeletalMesh>(FSoftObjectPath(SkeletalMeshPath).TryLoad());
	if (!Mesh)
	{
		return Result;
	}

	for (const UClothingAssetBase* Asset : Mesh->GetMeshClothingAssets())
	{
		if (Asset)
		{
			FClothingAssetInfo& Info = Result.AddDefaulted_GetRef();
			Info.AssetName = Asset->GetName();
			if (const UClothingAssetCommon* AssetCommon = Cast<UClothingAssetCommon>(Asset))
			{
				Info.bRequiresMatchingLodIndex = false;
				Info.NumClothingLods = AssetCommon->GetNumLods();
			}
		}
	}

	return Result;
}

FString UChaosClothAssetToolset::GetSectionClothing(
	const FString& SkeletalMeshPath, int32 LodIndex, int32 SectionIndex)
{
	using namespace UE::ClothAssetToolset::Private;

	FString Result;
	const USkeletalMesh* Mesh = Cast<USkeletalMesh>(FSoftObjectPath(SkeletalMeshPath).TryLoad());
	if (!Mesh)
	{
		RaiseError(FString::Printf(TEXT("Failed to load skeletal mesh '%s'"), *SkeletalMeshPath));
		return Result;
	}

	if (const UClothingAssetBase* Asset = Mesh->GetSectionClothingAsset(LodIndex, SectionIndex))
	{
		Result = Asset->GetName();
	}
	return Result;
}

FString UChaosClothAssetToolset::ConvertClothingAssetCommonToChaosClothAsset(
	const FString& SkeletalMeshPath,
	const FString& ClothingAssetName,
	const FString& OutputPackagePath,
	const FString& AssetName)
{
	using namespace UE::ClothAssetToolset::Private;

	const USkeletalMesh* const Mesh = Cast<USkeletalMesh>(FSoftObjectPath(SkeletalMeshPath).TryLoad());
	if (!Mesh)
	{
		RaiseError(FString::Printf(TEXT("Failed to load skeletal mesh '%s'"), *SkeletalMeshPath));
		return FString();
	}

	const UClothingAssetCommon* SourceAsset = nullptr;
	for (const UClothingAssetBase* Candidate : Mesh->GetMeshClothingAssets())
	{
		if (Candidate && Candidate->GetName() == ClothingAssetName)
		{
			if (Candidate->IsA<UChaosClothAssetBase>())
			{
				RaiseError(FString::Printf(
					TEXT("Clothing asset '%s' is already a UChaosClothAsset-derived type; nothing to convert."),
					*ClothingAssetName));
				return FString();
			}
			SourceAsset = Cast<UClothingAssetCommon>(Candidate);
			break;
		}
	}

	if (!SourceAsset)
	{
		RaiseError(FString::Printf(
			TEXT("No legacy UClothingAssetCommon named '%s' found on '%s'."),
			*ClothingAssetName, *SkeletalMeshPath));
		return FString();
	}

	if (OutputPackagePath.IsEmpty())
	{
		RaiseError(TEXT("OutputPackagePath must not be empty."));
		return FString();
	}

	const UE::Chaos::ClothAsset::FLegacyClothingConverterResult Result =
		UE::Chaos::ClothAsset::FLegacyClothingConverter::Convert(SourceAsset, OutputPackagePath, AssetName);

	if (!Result.CreatedAsset)
	{
		RaiseError(Result.ErrorText.IsEmpty()
			? TEXT("Conversion failed for unknown reasons.")
			: Result.ErrorText.ToString());
		return FString();
	}

	return Result.CreatedAssetPath;
}
