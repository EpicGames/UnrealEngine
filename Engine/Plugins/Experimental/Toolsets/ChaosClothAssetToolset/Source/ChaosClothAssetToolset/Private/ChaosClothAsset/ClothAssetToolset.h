// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

#include "ClothAssetToolset.generated.h"

/**
 * Information about a clothing asset on a skeletal mesh, returned by ListClothingAssets.
 */
USTRUCT(BlueprintType)
struct FClothingAssetInfo
{
	GENERATED_BODY()

	/** Name of the clothing asset. Pass to AssignClothingToSection. */
	UPROPERTY()
	FString AssetName;

	/**
	 * If true, ClothingLodIndex passed to AssignClothingToSection must match the skeletal mesh
	 * LodIndex. This is the case for ChaosClothAsset-derived types.
	 * If false (UClothingAssetCommon), any value in [0, NumClothingLods-1] is valid.
	 */
	UPROPERTY()
	bool bRequiresMatchingLodIndex = true;

	/**
	 * Number of LODs available in this clothing asset.
	 * Only relevant when bRequiresMatchingLodIndex is false.
	 */
	UPROPERTY()
	int32 NumClothingLods = 0;
};

/**
 * Provides tools for creating and assigning ChaosClothAsset clothing data to skeletal meshes.
 * Replicates the workflow available via the SkeletalMesh Editor viewport context menu.
 */
UCLASS(MinimalAPI)
class UChaosClothAssetToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	/**
	 * Creates a UChaosClothAssetSKMClothingAsset from a Chaos Cloth/Outfit Asset and adds it
	 * to the skeletal mesh's clothing asset array, making it available to assign to sections.
	 * @param SkeletalMeshPath Object path to the target USkeletalMesh asset.
	 * @param ChaosClothAssetPath Object path to the source UChaosClothAssetBase (cloth or outfit asset).
	 * @return List of created clothing assets, or empty if failed.
	 */
	UFUNCTION(meta=(AICallable), Category = "ChaosClothAsset")
	static TArray<FString> CreateClothingAsset(
		const FString& SkeletalMeshPath,
		const FString& ChaosClothAssetPath);

	/**
	 * Binds a clothing asset to a specific LOD and section on the skeletal mesh.
	 * Mirrors the SkeletalMesh Editor's "Apply Clothing Data" action.
	 * Any clothing already bound to the section is unbound first.
	 * @param SkeletalMeshPath Object path to the target USkeletalMesh asset.
	 * @param ClothingAssetName Name of the Clothing Asset on the mesh
	 *    (e.g., from CreateClothingAsset or ListClothingAssets).
	 * @param LodIndex Skeletal mesh LOD index to bind to.
	 * @param SectionIndex Section index within the LOD to bind to.
	 * @param ClothingLodIndex LOD index within the clothing asset to bind.
	 *    For ChaosClothAsset-derived types (bRequiresMatchingLodIndex == true in ListClothingAssets),
	 *    pass the same value as LodIndex. For UClothingAssetCommon, pass the desired clothing LOD
	 *    (0 to NumClothingLods-1 as reported by ListClothingAssets).
	 * @return True if the binding succeeded.
	 */
	UFUNCTION(meta=(AICallable), Category = "ChaosClothAsset")
	static bool AssignClothingToSection(
		const FString& SkeletalMeshPath,
		const FString& ClothingAssetName,
		int32 LodIndex,
		int32 SectionIndex,
		int32 ClothingLodIndex);

	/**
	 * Removes the clothing binding from a specific LOD section on the skeletal mesh.
	 * Mirrors the SkeletalMesh Editor's "Remove Clothing Data" action.
	 * @param SkeletalMeshPath Object path to the target USkeletalMesh asset.
	 * @param LodIndex Skeletal mesh LOD index.
	 * @param SectionIndex Section index within the LOD.
	 * @return True if a binding was found and removed.
	 */
	UFUNCTION(meta=(AICallable), Category = "ChaosClothAsset")
	static bool RemoveClothingFromSection(
		const FString& SkeletalMeshPath,
		int32 LodIndex,
		int32 SectionIndex);

	/**
	 * Returns information about all Clothing Assets on a skeletal mesh.
	 * Use AssetName with AssignClothingToSection.
	 * Check bRequiresMatchingLodIndex to determine how to supply ClothingLodIndex:
	 *   - true  (ChaosClothAsset): pass the same value as LodIndex
	 *   - false (UClothingAssetCommon): pass any value in [0, NumClothingLods-1]
	 * @param SkeletalMeshPath Object path to the target USkeletalMesh asset.
	 * @return Array of clothing asset info structs.
	 */
	UFUNCTION(meta=(AICallable), Category = "ChaosClothAsset")
	static TArray<FClothingAssetInfo> ListClothingAssets(const FString& SkeletalMeshPath);

	/**
	 * Returns the name of clothing bound for a specific LOD section (if any).
	 * @param SkeletalMeshPath Object path to the target USkeletalMesh asset.
	 * @param LodIndex Skeletal mesh LOD index.
	 * @param SectionIndex Section index within the LOD.
	 * @return The name of the clothing asset, or empty if none bound.
	 */
	UFUNCTION(meta=(AICallable), Category = "ChaosClothAsset")
	static FString GetSectionClothing(
		const FString& SkeletalMeshPath,
		int32 LodIndex,
		int32 SectionIndex);

	/**
	 * Converts a legacy UClothingAssetCommon (the cloth asset created via UChaosClothingSimulationFactory)
	 * into a new UChaosClothAsset whose embedded Dataflow graph mirrors the legacy UChaosClothConfig and
	 * UChaosClothSharedSimConfig values, with one WeightMap node per legacy weight map.
	 * @param SkeletalMeshPath  Object path to the USkeletalMesh that owns the legacy clothing asset.
	 * @param ClothingAssetName Name of the UClothingAssetCommon on the mesh (from ListClothingAssets).
	 * @param OutputPackagePath Folder where the new asset should be created (e.g. "/Game/Cloth/").
	 * @param AssetName         Name of the new asset; empty for default ("CA_Converted_<source>").
	 * @return Object path of the new UChaosClothAsset; empty on failure.
	 */
	UFUNCTION(meta=(AICallable), Category = "ChaosClothAsset")
	static FString ConvertClothingAssetCommonToChaosClothAsset(
		const FString& SkeletalMeshPath,
		const FString& ClothingAssetName,
		const FString& OutputPackagePath,
		const FString& AssetName);
};
